// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

public class AdpcmAudioDecoder : ModuleRules
{
    public AdpcmAudioDecoder(ReadOnlyTargetRules Target) : base(Target)
    {
        ShortName = "AdpcmAudioDecoder";

        PublicDependencyModuleNames.AddRange(
            new string[]
            {
                "Core",
                "CoreUObject",
                "Engine"
            }
        );
    }
}
