// Copyright Epic Games, Inc. All Rights Reserved.


#include "Decoders/ADPCMAudioInfo.h"
#include "Interfaces/IAudioFormat.h"
#include "Adpcm.h"

static int32 bDisableADPCMSeekingCVar = 0;
FAutoConsoleVariableRef CVarDisableADPCMSeeking(
	TEXT("au.adpcm.DisableSeeking"),
	bDisableADPCMSeekingCVar,
	TEXT("Disables seeking with ADPCM.\n"),
	ECVF_Default);

static int32 ADPCMReadFailiureTimeoutCVar = 64;
FAutoConsoleVariableRef CVarADPCMReadFailiureTimeout(
	TEXT("au.adpcm.ADPCMReadFailiureTimeout"),
	ADPCMReadFailiureTimeoutCVar,
	TEXT("Sets the number of ADPCM decode attempts we'll try before stopping the sound wave altogether.\n"),
	ECVF_Default);

static int32 ADPCMDisableSeekForwardOnChunkMissesCVar = 1;
FAutoConsoleVariableRef CVarADPCMDisableSeekForwardOnChunkMisses(
	TEXT("au.adpcm.DisableSeekForwardOnReadMisses"),
	ADPCMDisableSeekForwardOnChunkMissesCVar,
	TEXT("When there is a seek pending and this CVar is set to 0, we will scan forward in the file.\n"),
	ECVF_Default);

static int32 ADPCMOnlySeekForwardOneChunkCVar = 1;
FAutoConsoleVariableRef CVarADPCMOnlySeekForwardOneChunk(
	TEXT("au.adpcm.OnlySeekForwardOneChunk"),
	ADPCMOnlySeekForwardOneChunkCVar,
	TEXT("When set to 1, we will not continue to seek forward after failing to load two chunks in a row.\n"),
	ECVF_Default);

static float ChanceForIntentionalChunkMissCVar = 0.0f;
FAutoConsoleVariableRef CVarChanceForIntentionalChunkMiss(
	TEXT("au.adpcm.ChanceForIntentionalChunkMiss"),
	ChanceForIntentionalChunkMissCVar,
	TEXT("If this is set > 0 we will intentionally drop chunks. Used for debugging..\n"),
	ECVF_Default);

FADPCMAudioInfo::FADPCMAudioInfo(void)
	: NumConsecutiveReadFailiures(0)
	, UncompressedBlockSize(0)
	, CompressedBlockSize(0)
	, BlockSize(0)
	, StreamBufferSize(0)
	, TotalDecodedSize(0)
	, NumChannels(0)
	, Format(0)
	, PreviouslyRequestedChunkIndex(0)
	, UncompressedBlockData(nullptr)
	, SamplesPerBlock(0)
	, FirstChunkSampleDataOffset(0)
	, FirstChunkSampleDataIndex(0)
	, bDecompressorReleased(false)
	, bNewSeekRequest(false)
	, bSeekPendingRead(false)
	, bSeekedFowardToNextChunk(false)
	, TargetSeekFrame(0)
{
}

FADPCMAudioInfo::~FADPCMAudioInfo(void)
{
	if(UncompressedBlockData != nullptr)
	{
		FMemory::Free(UncompressedBlockData);
		UncompressedBlockData = nullptr;
	}
}

void FADPCMAudioInfo::SeekToTime(const float InSeekTime)
{
	if (bDisableADPCMSeekingCVar)
	{
		return;
	}

	const uint32 SamplesPerSec = *WaveInfo.pSamplesPerSec;
	TargetSeekFrame = static_cast<uint32>(InSeekTime * SamplesPerSec);
	bNewSeekRequest = true;
}

void FADPCMAudioInfo::SeekToFrame(const uint32 InSeekFrame)
{
	if (bDisableADPCMSeekingCVar)
	{
		return;
	}

	TargetSeekFrame = InSeekFrame;
	bNewSeekRequest = true;
}

void FADPCMAudioInfo::SeekToFrameInternal(const uint32 InSeekFrame)
{
	// Reset chunk handle in preperation for a new chunk.
	CurCompressedChunkData = nullptr;

	// zero out any existing decode data as it will stale after seeking
	if (UncompressedBlockData)
	{
		FMemory::Memzero(UncompressedBlockData, NumChannels * UncompressedBlockSize);
	}
	
	float SeekTimeSeconds = InSeekFrame / static_cast<float>(*WaveInfo.pSamplesPerSec);
	UE_LOGF(LogAudio, Verbose, "Seeking ADPCM source to %.3f sec", SeekTimeSeconds);

	if (InSeekFrame == 0)
	{
		CurrentCompressedBlockIndex = 0;
		// This is set to the max value to trigger the decompression of the first audio block
		CurrentUncompressedBlockSampleIndex = UncompressedBlockSize / sizeof(uint16);;
		CurrentChunkIndex = FirstChunkSampleDataIndex;
		CurrentChunkBufferOffset = FirstChunkSampleDataOffset;
		TotalSamplesStreamed = 0;
		
		ResetSeekState();
		return;
	}

	// Calculate block index & force SeekTime to be in bounds.
	check(WaveInfo.pSamplesPerSec != nullptr);
	check(TotalSamplesPerChannel > 1);

	uint32 SeekedSamples = InSeekFrame;
	TotalSamplesStreamed = FMath::Min<uint32>(SeekedSamples, TotalSamplesPerChannel - 1);

	const uint32 HeaderOffset = static_cast<uint32>(WaveInfo.SampleDataStart - SrcBufferData);
	
	if (!StreamingSoundWaveData)
	{
		// For the non streaming case:
		if (Format == WAVE_FORMAT_LPCM)
		{
			// There are no "blocks" on LPCM, so only update the total samples streamed (which is based off sample rate).
			// Note that TotalSamplesStreamed is per-channel in the ReadCompressedInfo. Channels are taken into account there.
			TotalSamplesStreamed = FMath::Clamp<uint32>(SeekedSamples, 0, TotalSamplesPerChannel - 1);
		}
		else
		{
			// Clamp to the end of memory in case we have an invalid seek time.
			SeekedSamples = FMath::Clamp<uint32>(SeekedSamples, 0, TotalSamplesPerChannel - 1);

			// Compute the block index that we're seeked to
			CurrentCompressedBlockIndex = SeekedSamples / SamplesPerBlock;

			// Update the samples streamed to the current block index and the samples per block
			TotalSamplesStreamed = CurrentCompressedBlockIndex * SamplesPerBlock;
		}
	}
	else
	{
		if (StreamingSoundWaveData->GetNumChunks() == 0)
		{
			UE_LOGF(LogAudio, Error, "Entered streaming seek path with a non-streaming sound!");
			return;
		}

		const uint32 TotalStreamingChunks = StreamingSoundWaveData->GetNumChunks();

		if (Format == WAVE_FORMAT_ADPCM)
		{
			CurrentCompressedBlockIndex = TotalSamplesStreamed / SamplesPerBlock; // Compute the block index that where SeekTime resides.
			CurrentChunkIndex = FirstChunkSampleDataIndex;
			CurrentChunkBufferOffset = FirstChunkSampleDataOffset;

			const int32 ChannelBlockSize = BlockSize * NumChannels;
			for (uint32 BlockIndex = 0; BlockIndex < CurrentCompressedBlockIndex; ++BlockIndex)
			{
				const uint32 SizeOfChunk = StreamingSoundWaveData->GetSizeOfChunk(CurrentChunkIndex);
				if (CurrentChunkBufferOffset + ChannelBlockSize >= SizeOfChunk)
				{
					const uint32 RemainderAfterEndOfChunk = CurrentChunkBufferOffset + ChannelBlockSize - SizeOfChunk;
					ensureMsgf(RemainderAfterEndOfChunk == 0, TEXT("Found partial ADPCM block of %u samples- Please check FADPCMAudioFormat::SplitDataForStreaming for errors."), RemainderAfterEndOfChunk);
					++CurrentChunkIndex;
					CurrentChunkBufferOffset = 0;
				}
				else
				{
					// Always add chunks in NumChannels pairs
					CurrentChunkBufferOffset += ChannelBlockSize;
				}
				
				if (CurrentChunkIndex >= TotalStreamingChunks)
				{
					CurrentChunkIndex = FirstChunkSampleDataIndex;
					CurrentChunkBufferOffset = FirstChunkSampleDataOffset;
					break;
				}
			}
		}
		else if (Format == WAVE_FORMAT_LPCM)
		{
			const int32 ChannelBlockSize = sizeof(int16) * NumChannels;
			const uint32 TotalByteSizeToSeek = TotalSamplesStreamed * ChannelBlockSize;	
			
			uint32 TargetChunkIndex = FirstChunkSampleDataIndex;
			uint32 SizeOfTargetChunk = StreamingSoundWaveData->GetSizeOfChunk(TargetChunkIndex);
			
			uint32 CurrentBytes = 0;
			while (CurrentBytes + SizeOfTargetChunk < TotalByteSizeToSeek)
			{
				const bool TargetChunkInRange = TargetChunkIndex < TotalStreamingChunks;
				ensure(TargetChunkInRange);

				if (!TargetChunkInRange)
				{
					break;
				}

				++TargetChunkIndex;
				CurrentBytes += SizeOfTargetChunk;
				CurrentChunkBufferOffset -= SizeOfTargetChunk;
				SizeOfTargetChunk = StreamingSoundWaveData->GetSizeOfChunk(TargetChunkIndex);
			}

			check(CurrentBytes <= TotalByteSizeToSeek);

			CurrentChunkIndex = TargetChunkIndex;
			CurrentChunkBufferOffset = TotalByteSizeToSeek - CurrentBytes;
			CurrentChunkBufferOffset -= CurrentChunkBufferOffset % ChannelBlockSize;
		}
		else
		{
			// If we hit this, Format was invalid:
			checkNoEntry();
			return;
		}
	}
	bSeekPendingRead = true;
}

bool FADPCMAudioInfo::ReadCompressedInfo(const uint8* InSrcBufferData, uint32 InSrcBufferDataSize, struct FSoundQualityInfo* QualityInfo)
{
	if (!InSrcBufferData)
	{
		FString Name = QualityInfo ? QualityInfo->DebugName : TEXT("Unknown");
		UE_LOGF(LogAudio, Error, "FADPCMAudioInfo::ReadCompressedInfo: Failed to read compressed ADPCM audio from ('%ls') because there was no resource data.", *Name);

		return false;
	}

	SrcBufferData = InSrcBufferData;
	SrcBufferDataSize = InSrcBufferDataSize;

	void*	FormatHeader;

	if (!WaveInfo.ReadWaveInfo((uint8*)SrcBufferData, SrcBufferDataSize, nullptr, false, &FormatHeader))
	{
		UE_LOGF(LogAudio, Error, "FADPCMAudioInfo::ReadCompressedInfo: WaveInfo.ReadWaveInfo Failed");
		return false;
	}

	Format = *WaveInfo.pFormatTag;
	NumChannels = *WaveInfo.pChannels;

	if (Format == WAVE_FORMAT_ADPCM)
	{
		ADPCM::ADPCMFormatHeader* ADPCMHeader = (ADPCM::ADPCMFormatHeader*)FormatHeader;
		TotalSamplesPerChannel = ADPCMHeader->SamplesPerChannel;
		SamplesPerBlock = ADPCMHeader->wSamplesPerBlock;

		const uint32 PreambleSize = 7;
		BlockSize = *WaveInfo.pBlockAlign;

		// ADPCM starts with 2 uncompressed samples and then the remaining compressed sample data has 2 samples per byte
		UncompressedBlockSize = (2 + (BlockSize - PreambleSize) * 2) * sizeof(int16);
		CompressedBlockSize = BlockSize;

		const uint32 uncompressedBlockSamples = (2 + (BlockSize - PreambleSize) * 2);
		const uint32 targetBlocks = MONO_PCM_BUFFER_SAMPLES / uncompressedBlockSamples;
		StreamBufferSize = targetBlocks * UncompressedBlockSize;
		// Ensure TotalDecodedSize is a even multiple of the compressed block size so that the buffer is not over read on the last block
		TotalDecodedSize = ((WaveInfo.SampleDataSize + CompressedBlockSize - 1) / CompressedBlockSize) * UncompressedBlockSize;

		UncompressedBlockData = (uint8*)FMemory::Realloc(UncompressedBlockData, NumChannels * UncompressedBlockSize);
		check(UncompressedBlockData != nullptr);
		TotalCompressedBlocksPerChannel = (WaveInfo.SampleDataSize + CompressedBlockSize - 1) / CompressedBlockSize / NumChannels;
	}
	else if(Format == WAVE_FORMAT_LPCM)
	{
		// There are no "blocks" in this case
		BlockSize = 0;
		UncompressedBlockSize = 0;
		CompressedBlockSize = 0;
		StreamBufferSize = 0;
		UncompressedBlockData = nullptr;
		TotalCompressedBlocksPerChannel = 0;

		TotalDecodedSize = WaveInfo.SampleDataSize;

		TotalSamplesPerChannel = TotalDecodedSize / sizeof(uint16) / NumChannels;
	}
	else
	{
		UE_LOGF(LogAudio, Error, "FADPCMAudioInfo::ReadCompressedInfo: Invalid format. Expected %d or %d. Got %d", WAVE_FORMAT_LPCM, WAVE_FORMAT_ADPCM, Format);
		return false;
	}

	if (QualityInfo)
	{
		QualityInfo->SampleRate = *WaveInfo.pSamplesPerSec;
		QualityInfo->NumChannels = *WaveInfo.pChannels;
		QualityInfo->SampleDataSize = TotalDecodedSize;
		QualityInfo->Duration = (float)TotalSamplesPerChannel / QualityInfo->SampleRate;
	}

	CurrentCompressedBlockIndex = 0;
	TotalSamplesStreamed = 0;
	// This is set to the max value to trigger the decompression of the first audio block
	CurrentUncompressedBlockSampleIndex = UncompressedBlockSize / sizeof(uint16);

	return true;
}

bool FADPCMAudioInfo::ReadCompressedData(uint8* Destination, bool bLooping, uint32 BufferSize)
{

	const uint32 ChannelSampleSize = sizeof(uint16) * NumChannels;

	// This correctly handles any BufferSize as long as its a multiple of sample size * number of channels
	check(Destination);
	check(BufferSize % ChannelSampleSize == 0);

	ProcessSeekRequest();

	// If we've already read through this asset and we are not looping, memzero and early out.
	if (TotalSamplesStreamed >= TotalSamplesPerChannel && !bLooping)
	{
		FMemory::Memzero(Destination, BufferSize);
		return true;
	}

	int16* OutData = (int16*)Destination;
	bool ReachedEndOfSamples = false;
	if(Format == WAVE_FORMAT_ADPCM)
	{
		// We need to loop over the number of samples requested since an uncompressed block will not match the number of frames requested
		while(BufferSize > 0)
		{
			if(CurrentUncompressedBlockSampleIndex >= UncompressedBlockSize / sizeof(uint16))
			{
				// we need to decompress another block of compressed data from the current chunk

				// Decompress one block for each channel and store it in UncompressedBlockData
				for(int32 ChannelItr = 0; ChannelItr < NumChannels; ++ChannelItr)
				{
					bool bSuccess = ADPCM::DecodeBlock(
						WaveInfo.SampleDataStart + (ChannelItr * TotalCompressedBlocksPerChannel + CurrentCompressedBlockIndex) * CompressedBlockSize,
						CompressedBlockSize,
						(int16*)(UncompressedBlockData + ChannelItr * UncompressedBlockSize));

					if (!bSuccess)
					{
						FMemory::Memzero(Destination, BufferSize);
						bHasError = true;
						return true;
					}
				}

				// Update some bookkeeping
				CurrentUncompressedBlockSampleIndex = 0;
				++CurrentCompressedBlockIndex;
			}

			// Only copy over the number of samples we currently have available, we will loop around if needed
			uint32 DecompressedSamplesToCopy = FMath::Min<uint32>(UncompressedBlockSize / sizeof(uint16) - CurrentUncompressedBlockSampleIndex, BufferSize / (sizeof(uint16) * NumChannels));
			check(DecompressedSamplesToCopy > 0);

			// Ensure we don't go over the number of samples left in the audio data
			if(DecompressedSamplesToCopy > TotalSamplesPerChannel - TotalSamplesStreamed)
			{
				DecompressedSamplesToCopy = TotalSamplesPerChannel - TotalSamplesStreamed;
			}

			// Copy over the actual sample data
			for(uint32 SampleItr = 0; SampleItr < DecompressedSamplesToCopy; ++SampleItr)
			{
				for(int32 ChannelItr = 0; ChannelItr < NumChannels; ++ChannelItr)
				{
					uint16 Value = *(int16*)(UncompressedBlockData + ChannelItr * UncompressedBlockSize + (CurrentUncompressedBlockSampleIndex + SampleItr) * sizeof(int16));
					*OutData++ = Value;
				}
			}

			// Update bookkeeping
			CurrentUncompressedBlockSampleIndex += DecompressedSamplesToCopy;
			BufferSize -= DecompressedSamplesToCopy * sizeof(uint16) * NumChannels;
			TotalSamplesStreamed += DecompressedSamplesToCopy;

			// Check for the end of the audio samples and loop if needed
			if(TotalSamplesStreamed >= TotalSamplesPerChannel)
			{
				ReachedEndOfSamples = true;
				if(!bLooping)
				{
					// Zero remaining buffer
					FMemory::Memzero(OutData, BufferSize);
					
					ResetSeekState();
					
					return true;
				}
				else
				{
					// This is set to the max value to trigger the decompression of the first audio block
					CurrentUncompressedBlockSampleIndex = UncompressedBlockSize / sizeof(uint16);
					CurrentCompressedBlockIndex = 0;
					TotalSamplesStreamed = 0;
				}
			}
		}
	}
	else
	{
		uint32 OutDataOffset = 0;
		while (BufferSize > 0)
		{
			uint32 DecompressedSamplesToCopy = BufferSize / ChannelSampleSize;

			// Ensure we don't go over the number of samples left in the audio data
			if (DecompressedSamplesToCopy > TotalSamplesPerChannel - TotalSamplesStreamed)
			{
				DecompressedSamplesToCopy = TotalSamplesPerChannel - TotalSamplesStreamed;
			}

			FMemory::Memcpy(OutData + OutDataOffset, WaveInfo.SampleDataStart + (TotalSamplesStreamed * ChannelSampleSize), DecompressedSamplesToCopy * ChannelSampleSize);
			TotalSamplesStreamed += DecompressedSamplesToCopy;
			BufferSize -= DecompressedSamplesToCopy * ChannelSampleSize;
			OutDataOffset += DecompressedSamplesToCopy * NumChannels;

			// Check for the end of the audio samples and loop if needed
			if (TotalSamplesStreamed >= TotalSamplesPerChannel)
			{
				ReachedEndOfSamples = true;
				TotalSamplesStreamed = 0;
				if (!bLooping)
				{
					// Zero remaining buffer
					FMemory::Memzero(OutData + OutDataOffset, BufferSize);

					ResetSeekState();

					return true;
				}
			}
		}
	}

	ResetSeekState();

	return ReachedEndOfSamples;
}

void FADPCMAudioInfo::ExpandFile(uint8* DstBuffer, struct FSoundQualityInfo* QualityInfo)
{
	check(DstBuffer);

	ReadCompressedData(DstBuffer, false, TotalDecodedSize);
}

int FADPCMAudioInfo::GetStreamBufferSize() const
{
	return StreamBufferSize;
}

void FADPCMAudioInfo::ProcessSeekRequest()
{
	if (bNewSeekRequest)
	{
		SeekToFrameInternal(TargetSeekFrame);
	}
}

void FADPCMAudioInfo::ResetSeekState()
{
	bSeekPendingRead = false;
	bNewSeekRequest = false;
}

bool FADPCMAudioInfo::StreamCompressedInfoInternal(const TSharedRef<const FSoundWaveData>& InWaveData, struct FSoundQualityInfo* QualityInfo)
{
	FScopeLock ScopeLock(&CurCompressedChunkHandleCriticalSection);

	check(QualityInfo);
	check(StreamingSoundWaveData == InWaveData);

	CurrentChunkIndex = 0;

	// Get the first chunk of audio data (should already be loaded)
	uint8 const* ChunkData = GetLoadedChunk(*InWaveData, CurrentChunkIndex, CurrentChunkDataSize);

	if (ChunkData == nullptr)
	{
		UE_LOGF(LogAudio, Warning, "FADPCMAudioInfo::StreamCompressedInfoInternal: Failed to get loaded chunk at index '%u'", CurrentChunkIndex);
		return false;
	}

	SrcBufferData = nullptr;
	SrcBufferDataSize = 0;

	void* FormatHeader = nullptr;
	FString ErrorMsg;
	constexpr bool bHeaderDataOnly = true;
	if (!WaveInfo.ReadWaveInfo(static_cast<const uint8*>(ChunkData), CurrentChunkDataSize, &ErrorMsg, bHeaderDataOnly, &FormatHeader))
	{
		UE_LOGF(LogAudio, Warning, "FADPCMAudioInfo::StreamCompressedInfoInternal: Failed to read WaveInfo: %ls", *ErrorMsg);
		return false;
	}
	
	// if we only included the header in the zeroth chunk, skip to the next chunk.
	const int32 SampleDataOffset = UE_PTRDIFF_TO_INT32(WaveInfo.SampleDataStart - ChunkData);
	check(SampleDataOffset > 0);
	if (((uint32)SampleDataOffset) >= CurrentChunkDataSize)
	{
		++CurrentChunkIndex;
		ChunkData = GetLoadedChunk(*InWaveData, CurrentChunkIndex, CurrentChunkDataSize);
		FirstChunkSampleDataIndex = CurrentChunkIndex;
		FirstChunkSampleDataOffset = 0;
	}
	else
	{
		FirstChunkSampleDataOffset = UE_PTRDIFF_TO_INT32(WaveInfo.SampleDataStart - ChunkData);
		FirstChunkSampleDataIndex = 0;
	}

	PreviouslyRequestedChunkIndex = CurrentChunkIndex;

	SrcBufferData = ChunkData;
	CurrentChunkBufferOffset = 0;
	CurCompressedChunkData = nullptr;
	bDecompressorReleased = false;
	CurrentUncompressedBlockSampleIndex = 0;
	
	TotalSamplesStreamed = 0;
	Format = *WaveInfo.pFormatTag;
	NumChannels = *WaveInfo.pChannels;

	if (Format == WAVE_FORMAT_ADPCM)
	{
		check(FormatHeader);
		ADPCM::ADPCMFormatHeader* ADPCMHeader = (ADPCM::ADPCMFormatHeader*)FormatHeader;
		TotalSamplesPerChannel = ADPCMHeader->SamplesPerChannel;
		SamplesPerBlock = ADPCMHeader->wSamplesPerBlock;

		const uint32 PreambleSize = 7;

		BlockSize = *WaveInfo.pBlockAlign;
		UncompressedBlockSize = (2 + (BlockSize - PreambleSize) * 2) * sizeof(int16);
		CompressedBlockSize = BlockSize;

		// Calculate buffer sizes and total number of samples
		const uint32 uncompressedBlockSamples = (2 + (BlockSize - PreambleSize) * 2);
		const uint32 targetBlocks = MONO_PCM_BUFFER_SAMPLES / uncompressedBlockSamples;
		StreamBufferSize = targetBlocks * UncompressedBlockSize;
		TotalDecodedSize = ((WaveInfo.SampleDataSize + CompressedBlockSize - 1) / CompressedBlockSize) * UncompressedBlockSize;

		UncompressedBlockData = (uint8*)FMemory::Realloc(UncompressedBlockData, NumChannels * UncompressedBlockSize);
		check(UncompressedBlockData != nullptr);
	}
	else if (Format == WAVE_FORMAT_LPCM)
	{
		BlockSize = 0;
		UncompressedBlockSize = 0;
		CompressedBlockSize = 0;

		// This is uncompressed, so decoded size and buffer size are the same
		TotalDecodedSize = WaveInfo.SampleDataSize;
		StreamBufferSize = WaveInfo.SampleDataSize;
		TotalSamplesPerChannel = StreamBufferSize / sizeof(uint16) / NumChannels;
	}
	else
	{
		UE_LOGF(LogAudio, Error, "FADPCMAudioInfo::StreamCompressedInfoInternal: Unsupported wave format (Neither ADPCM nor LPCM)");
		return false;
	}

	if (QualityInfo)
	{
		QualityInfo->SampleRate = *WaveInfo.pSamplesPerSec;
		QualityInfo->NumChannels = *WaveInfo.pChannels;
		QualityInfo->SampleDataSize = TotalDecodedSize;
		QualityInfo->Duration = (float)TotalSamplesPerChannel / QualityInfo->SampleRate;
	}

	return true;
}

bool FADPCMAudioInfo::StreamCompressedData(uint8* Destination, bool bLooping, uint32 BufferSize, int32& OutNumBytesStreamed)
{
	FScopeLock ScopeLock(&CurCompressedChunkHandleCriticalSection);
	int32 OriginalBufferSize = (int32)BufferSize;
	OutNumBytesStreamed = 0;

	// Initial sanity checks:
	if (bDecompressorReleased)
	{
		UE_LOGF(LogAudio, Error, "Stream Compressed Data called after release!");
		if (Destination != nullptr)
		{
			FMemory::Memzero(Destination, BufferSize);
		}
		return true;
	}

	if (Destination == nullptr || BufferSize == 0)
	{
		UE_LOGF(LogAudio, Error, "Stream Compressed Info not called!");
		return false;
	}

	if (NumChannels == 0)
	{
		UE_LOGF(LogAudio, Error, "Stream Compressed Info not called!");
		FMemory::Memzero(Destination, BufferSize);
		return true;
	}

	// Destination samples are interlaced by channel, BufferSize is in bytes
	const int32 ChannelSampleSize = sizeof(uint16) * NumChannels;

	// Ensure that BuffserSize is a multiple of the sample size times the number of channels
	if (BufferSize % ChannelSampleSize != 0)
	{
		UE_LOGF(LogAudio, Error, "Invalid buffer size %d requested for %d channels.", BufferSize, NumChannels);
		FMemory::Memzero(Destination, BufferSize);
		return true;
	}

	ProcessSeekRequest();

	int16* OutData = (int16*)Destination;
	bool ReachedEndOfSamples = false;
	if(Format == WAVE_FORMAT_ADPCM)
	{
		// We need to loop over the number of samples requested since an uncompressed block will not match the number of frames requested
		while(BufferSize > 0)
		{
			if(CurCompressedChunkData == nullptr || CurrentUncompressedBlockSampleIndex >= UncompressedBlockSize / sizeof(uint16))
			{
				// we need to decompress another block of compressed data from the current chunk

				if(CurCompressedChunkData == nullptr || CurrentChunkBufferOffset >= CurrentChunkDataSize)
				{
					// Get another chunk of compressed data from the streaming engine

					// CurrentChunkIndex is used to keep track of the current chunk for loading/unloading by the streaming engine, but chunk 0 is
					// preloaded so don't increment this when getting chunk 0, only later chunks. If previous chunk retrieval failed because it was
					// not ready, don't re-increment the CurrentChunkIndex.  Only increment if seek not pending as seek determines the current chunk index
					// and invalidates CurCompressedChunkData.
					if(CurCompressedChunkData != nullptr)
					{
						// If we have already seeked forward to the next chunk because the chunk didn't load in time for the render callback, we don't increment here.
						// Otherwise, we increment to the next chunk index and reset our chunk offset to the beginning of the new chunk.
						if (!bSeekedFowardToNextChunk)
						{
							// Ensure that we're either seeking or moving sequentially through the file.
							check(bSeekPendingRead || CurrentChunkIndex == PreviouslyRequestedChunkIndex);
							++CurrentChunkIndex;

							// Set the current buffer offset accounting for the header in the first chunk
							if (!bSeekPendingRead)
							{
								// Ensure that, if we hit this code, we are not at the beginning of the file.
								check(CurrentChunkIndex != FirstChunkSampleDataIndex);
								CurrentChunkBufferOffset = 0;
							}
						}

						// If this flag was raised, clear it for the next time we move on to another chunk.
						bSeekedFowardToNextChunk = false;
					}

					if (CurrentChunkIndex >= StreamingSoundWaveData->GetNumChunks())
					{
						// If this is the case, we've seeked to the end of the file.
						ReachedEndOfSamples = true;
						CurrentUncompressedBlockSampleIndex = 0;
						CurrentChunkIndex = FirstChunkSampleDataIndex;
						CurrentChunkBufferOffset = FirstChunkSampleDataOffset;
						TotalSamplesStreamed = 0;
						CurCompressedChunkData = nullptr;
						if (!bLooping)
						{
							// Set the remaining buffer to 0
							FMemory::Memset(OutData, 0, BufferSize);
							return true;
						}
					}

					// Request the next chunk of data from the streaming engine
					CurCompressedChunkData = GetLoadedChunk(*StreamingSoundWaveData, CurrentChunkIndex, CurrentChunkDataSize);

					if(CurCompressedChunkData == nullptr)
					{
						// We only need to worry about missing the stream chunk if we were seeking. Seeking might cause a bit of latency with chunk loading. That is expected.
						if (!bSeekPendingRead)
						{
							// If we did not load chunk then bail. CurrentChunkIndex will not get incremented on the next callback so in effect another attempt will be made to fetch the chunk.
							// Since audio streaming depends on the general data streaming mechanism used by other parts of the engine and new data is pre-fetched on the game thread, its
							// possible a game thread stall can cause this.
							UE_LOGF(LogAudio, Verbose, "Missed Deadline chunk %d", CurrentChunkIndex);
						}

						// If we have a seek pending, or we're in the middle of playing back the file, seek forward in the chunk offset to where we would have been.
						// If we've already seeked forward one chunk and haven't loaded the next chunk in time and are skipping to the next one, we stop seeking to prevent snowballing chunk load requests.
						const bool bAlreadySeekedFowardOneChunk = ADPCMOnlySeekForwardOneChunkCVar && bSeekedFowardToNextChunk;
						const bool bShouldSeekForwardOnMissedChunk = !ADPCMDisableSeekForwardOnChunkMissesCVar && !bAlreadySeekedFowardOneChunk && (bSeekPendingRead || CurrentChunkIndex > FirstChunkSampleDataIndex);

						if (bShouldSeekForwardOnMissedChunk)
						{
							const uint32 NumBlocksForCallback = BufferSize / (UncompressedBlockSize * NumChannels);
							const uint32 NumCompressedBytesForCallback = NumBlocksForCallback * CompressedBlockSize * NumChannels;
							
							uint32 NumDecompressedSamplesForCallback = BufferSize / (ChannelSampleSize);

							if (NumDecompressedSamplesForCallback > TotalSamplesPerChannel - TotalSamplesStreamed)
							{
								// If this is the case, we've seeked to the end of the file.
								ReachedEndOfSamples = true;
								CurrentUncompressedBlockSampleIndex = 0;
								CurrentChunkIndex = FirstChunkSampleDataIndex;
								CurrentChunkBufferOffset = FirstChunkSampleDataOffset;
								TotalSamplesStreamed = 0;
								CurCompressedChunkData = nullptr;
								if (!bLooping)
								{
									// Set the remaining buffer to 0
									FMemory::Memset(OutData, 0, BufferSize);
									return true;
								}
							}

							// Seek forward in the file the amount we would be rendering otherwise.
							CurrentChunkBufferOffset += NumCompressedBytesForCallback;
							TotalSamplesStreamed += NumDecompressedSamplesForCallback;

							const uint32 SizeOfCurrentChunk = StreamingSoundWaveData->GetSizeOfChunk(CurrentChunkIndex);
							if (CurrentChunkBufferOffset >= SizeOfCurrentChunk)
							{
								// If this is the case, we've seeked forward past the end of the chunk.

								// Ensure we never hit this section of code twice, resulting in a double increment of the chunk index.
								ensureAlways(!bSeekedFowardToNextChunk);

								// Ensure that we are either seeking of moving sequentially through the file.
								ensureAlwaysMsgf(bSeekPendingRead || CurrentChunkIndex == PreviouslyRequestedChunkIndex, TEXT("Failed to load a chunk of ADPCM audio for the entire duration that chunk of audio was supposed to be played."));
								++CurrentChunkIndex;
								CurrentChunkBufferOffset -= SizeOfCurrentChunk;

								// Ensure that we have not moved past the size of our new chunk with our current callback.
								ensureAlways(CurrentChunkBufferOffset < StreamingSoundWaveData->GetSizeOfChunk(CurrentChunkIndex));

								bSeekedFowardToNextChunk = true;

								if (!ensureMsgf(CurrentChunkBufferOffset % CompressedBlockSize == 0, TEXT("Error: seeked partway into an ADPCM block. Please check the above code, as well as FAudioFormatADPCM::SplitDataForStreaming for accuracy.")))
								{
									CurrentChunkBufferOffset = AlignDown(CurrentChunkBufferOffset, CompressedBlockSize);
								}

								if (CurrentChunkIndex >= StreamingSoundWaveData->GetNumChunks())
								{
									// If this is the case, we've seeked to the end of the file.
									ReachedEndOfSamples = true;
									CurrentUncompressedBlockSampleIndex = 0;
									CurrentChunkIndex = FirstChunkSampleDataIndex;
									CurrentChunkBufferOffset = FirstChunkSampleDataOffset;
									TotalSamplesStreamed = 0;
									CurCompressedChunkData = nullptr;
									if (!bLooping)
									{
										// Set the remaining buffer to 0
										FMemory::Memset(OutData, 0, BufferSize);
										return true;
									}
								}
							}
						}

						// zero out remaining data and bail
						FMemory::Memset(OutData, 0, BufferSize);
						return false;
					}

					
					if (CurrentChunkIndex == FirstChunkSampleDataIndex && !bSeekPendingRead)
					{
						// If we're in the first chunk, set the current buffer offset accounting for the header.
						CurrentChunkBufferOffset = FirstChunkSampleDataOffset;
					}

					ResetSeekState();
				}

				// Decompress one block for each channel and store it in UncompressedBlockData
				for(int32 ChannelItr = 0; ChannelItr < NumChannels; ++ChannelItr)
				{
					bool bSuccess = ADPCM::DecodeBlock(
						CurCompressedChunkData + CurrentChunkBufferOffset + ChannelItr * CompressedBlockSize,
						CompressedBlockSize,
						(int16*)(UncompressedBlockData + ChannelItr * UncompressedBlockSize));

					if (!bSuccess)
					{
						FMemory::Memzero(Destination, BufferSize);
						bHasError = true;
						return true;
					}
				}

				// Update some bookkeeping
				CurrentUncompressedBlockSampleIndex = 0;
				CurrentChunkBufferOffset += NumChannels * CompressedBlockSize;
			}

			// Only copy over the number of samples we currently have available, we will loop around if needed
			uint32 DecompressedSamplesToCopy = FMath::Min<uint32>(
				(UncompressedBlockSize / sizeof(uint16)) - CurrentUncompressedBlockSampleIndex,
				BufferSize / (ChannelSampleSize));
			check(DecompressedSamplesToCopy > 0);

			// Ensure we don't go over the number of samples left in the audio data
			if(DecompressedSamplesToCopy > TotalSamplesPerChannel - TotalSamplesStreamed)
			{
				DecompressedSamplesToCopy = TotalSamplesPerChannel - TotalSamplesStreamed;
			}

			// Copy over the actual sample data
			for(uint32 SampleItr = 0; SampleItr < DecompressedSamplesToCopy; ++SampleItr)
			{
				for(int32 ChannelItr = 0; ChannelItr < NumChannels; ++ChannelItr)
				{
					uint16 Value = *(int16*)(UncompressedBlockData + ChannelItr * UncompressedBlockSize + (CurrentUncompressedBlockSampleIndex + SampleItr) * sizeof(int16));
					*OutData++ = Value;
					OutNumBytesStreamed += sizeof(int16);
				}
			}

			// Update bookkeeping
			CurrentUncompressedBlockSampleIndex += DecompressedSamplesToCopy;
			BufferSize -= DecompressedSamplesToCopy * ChannelSampleSize;
			TotalSamplesStreamed += DecompressedSamplesToCopy;

			// Check for the end of the audio samples and loop if needed
			if(TotalSamplesStreamed >= TotalSamplesPerChannel)
			{
				ReachedEndOfSamples = true;
				CurrentUncompressedBlockSampleIndex = 0;
				CurrentChunkIndex = FirstChunkSampleDataIndex;
				CurrentChunkBufferOffset = 0;
				TotalSamplesStreamed = 0;
				CurCompressedChunkData = nullptr;
				if(!bLooping)
				{
					// Set the remaining buffer to 0
					FMemory::Memset(OutData, 0, BufferSize);
					return true;
				}
			}
		}
	}
	else
	{
		while(BufferSize > 0)
		{
			if(CurCompressedChunkData == nullptr || CurrentChunkBufferOffset >= CurrentChunkDataSize)
			{
				// Get another chunk of compressed data from the streaming engine

				// CurrentChunkIndex is used to keep track of the current chunk for loading/unloading by the streaming engine but chunk 0
				// is preloaded so we don't want to increment this when getting chunk 0, only later chunks
				// Also, if we failed to get a previous chunk because it was not ready we don't want to re-increment the CurrentChunkIndex
				if(CurCompressedChunkData != nullptr)
				{
					++CurrentChunkIndex;
				}

				// Check for the end of the audio samples and loop if needed
				if (CurrentChunkIndex >= StreamingSoundWaveData->GetNumChunks())
				{
					ReachedEndOfSamples = true;
					CurrentChunkIndex = FirstChunkSampleDataIndex;
					CurrentChunkBufferOffset = FirstChunkSampleDataOffset;
					TotalSamplesStreamed = 0;
					CurCompressedChunkData = nullptr;
					if (!bLooping)
					{
						// Set the remaining buffer to 0
						FMemory::Memset(OutData, 0, BufferSize);
						return true;
					}
				}

				// Request the next chunk of data from the streaming engine
				CurCompressedChunkData = GetLoadedChunk(*StreamingSoundWaveData, CurrentChunkIndex, CurrentChunkDataSize);

				if(CurCompressedChunkData == nullptr)
				{
					// Only report missing the stream chunk if we were seeking. Seeking
					// may cause a bit of latency with chunk loading, which is expected.
					if (!bSeekPendingRead)
					{
						// CurrentChunkIndex will not get incremented on the next callback, effectively causing another attempt to fetch the chunk.
						// Since audio streaming depends on the general data streaming mechanism used by other parts of the engine and new data is
						// prefetched on the game tick thread, this may be caused by a game hitch.
						UE_LOGF(LogAudio, Verbose, "Missed streaming ADPCM deadline chunk %d", CurrentChunkIndex);
					}

					FMemory::Memset(OutData, 0, BufferSize);
					NumConsecutiveReadFailiures++;
					const bool bReadAttemptTimedOut = NumConsecutiveReadFailiures > ADPCMReadFailiureTimeoutCVar;
					UE_CLOGF(bReadAttemptTimedOut, LogAudio, Warning, "ADPCM Audio Decode timed out.");
					return NumConsecutiveReadFailiures > ADPCMReadFailiureTimeoutCVar;
				}

				NumConsecutiveReadFailiures = 0;

				// Set the current buffer offset accounting for the header in the first chunk
				if (!bSeekPendingRead)
				{
					CurrentChunkBufferOffset = CurrentChunkIndex == FirstChunkSampleDataIndex ? FirstChunkSampleDataOffset : 0;
				}

				ResetSeekState();
			}
			
			uint32 DecompressedSamplesToCopy = FMath::Min<uint32>(
				(CurrentChunkDataSize - CurrentChunkBufferOffset) / ChannelSampleSize,
				BufferSize / ChannelSampleSize);

			if (DecompressedSamplesToCopy == 0)
			{
				CurrentChunkBufferOffset = CurrentChunkDataSize;
				continue;
			}

			// Ensure we don't go over the number of samples left in the audio data
			if(DecompressedSamplesToCopy > TotalSamplesPerChannel - TotalSamplesStreamed)
			{
				DecompressedSamplesToCopy = TotalSamplesPerChannel - TotalSamplesStreamed;
			}

			const uint32 SampleSize = DecompressedSamplesToCopy * ChannelSampleSize;
			FMemory::Memcpy(OutData, CurCompressedChunkData + CurrentChunkBufferOffset, SampleSize);
			OutData += DecompressedSamplesToCopy * NumChannels;
			OutNumBytesStreamed += DecompressedSamplesToCopy * NumChannels * sizeof(int16);

			CurrentChunkBufferOffset += SampleSize;
			BufferSize -= SampleSize;
			TotalSamplesStreamed += DecompressedSamplesToCopy;

			// Check for the end of the audio samples and loop if needed
			if(TotalSamplesStreamed >= TotalSamplesPerChannel)
			{
				ReachedEndOfSamples = true;
				CurrentChunkIndex = FirstChunkSampleDataIndex;
				CurrentChunkBufferOffset = FirstChunkSampleDataOffset;
				TotalSamplesStreamed = 0;
				CurCompressedChunkData = nullptr;
				if(!bLooping)
				{
					// Set the remaining buffer to 0
					FMemory::Memset(OutData, 0, BufferSize);
					return true;
				}
			}
		}
	}

	return ReachedEndOfSamples;
}

bool FADPCMAudioInfo::ReleaseStreamChunk(bool bBlockUntilReleased)
{
	ResetSeekState();

	if (bBlockUntilReleased)
	{
		// Wait for any pending decode tasks to finish.
		FScopeLock ScopeLock(&CurCompressedChunkHandleCriticalSection);
		CurCompressedChunkHandle = FAudioChunkHandle();
		CurrentChunkBufferOffset = 0;
		CurCompressedChunkData = nullptr;
		CurrentChunkDataSize = 0;
		bDecompressorReleased = true;
		return true;
	}
	else
	{
		// If the chunk isn't currently in use, reset the chunk handle.
		if (CurCompressedChunkHandleCriticalSection.TryLock())
		{
			CurCompressedChunkHandle = FAudioChunkHandle();
			CurrentChunkBufferOffset = 0;
			CurCompressedChunkData = nullptr;
			CurrentChunkDataSize = 0;
			bDecompressorReleased = true;
			CurCompressedChunkHandleCriticalSection.Unlock();
			return true;
		}
		else
		{
			// Otherwise, do nothing and exit.
			return false;
		}
	}
}

const uint8* FADPCMAudioInfo::GetLoadedChunk(const FSoundWaveData& InSoundWaveData, uint32 ChunkIndex, uint32& OutChunkSize)
{
	if (ChunkIndex != 0 && FMath::RandRange(0.0f, 1.0f) < ChanceForIntentionalChunkMissCVar)
	{
		UE_LOGF(LogAudio, Display, "Intentionally dropping a chunk here.");
		OutChunkSize = 0;
		return nullptr;
	}

	if (!ensure(ChunkIndex < InSoundWaveData.GetNumChunks()))
	{
		OutChunkSize = 0;
		return nullptr;
	}
	else if (ChunkIndex == 0)
	{
		TArrayView<const uint8> ZerothChunk = InSoundWaveData.GetZerothChunk(true);
		OutChunkSize = ZerothChunk.Num();
		PreviouslyRequestedChunkIndex = ChunkIndex;
		return ZerothChunk.GetData();
	}
	else
	{
		const bool bIsSeekingOrLooping = bSeekPendingRead || ChunkIndex == FirstChunkSampleDataIndex;
		ensureAlwaysMsgf(bIsSeekingOrLooping || ChunkIndex == PreviouslyRequestedChunkIndex || ChunkIndex == PreviouslyRequestedChunkIndex + 1, TEXT("ADPCM playback error! We skipped from the end of chunk %d to the beginning of chunk %d."), PreviouslyRequestedChunkIndex, ChunkIndex);

		CurCompressedChunkHandle = IStreamingManager::Get().GetAudioStreamingManager().GetLoadedChunk(InSoundWaveData.AsShared(), ChunkIndex, false, true);
		OutChunkSize = CurCompressedChunkHandle.Num();

		PreviouslyRequestedChunkIndex = ChunkIndex;
		
		return CurCompressedChunkHandle.GetData();
	}
}
