// Copyright Epic Games, Inc. All Rights Reserved.

#include "AudioDecompress.h"
#include "Decoders/ADPCMAudioInfo.h"
#include "PcmAudioInfoHybrid.h"

class ADPCMAUDIODECODER_API FAdpcmAudioDecoderModule : public IModuleInterface
{
public:	
	TUniquePtr<IAudioInfoFactory> PcmFactory;
	TUniquePtr<IAudioInfoFactory> AdpcmFactory;

	template<typename T> TUniquePtr<IAudioInfoFactory> MakeFactory(const FName Name)
	{
		return MakeUnique<FSimpleAudioInfoFactory>([] { return new T(); }, Name);
	}
	void RegisterLegacy()
	{
		AdpcmFactory = MakeFactory<FADPCMAudioInfo>(Audio::NAME_ADPCM);
		PcmFactory = MakeFactory<FADPCMAudioInfo>(Audio::NAME_PCM);
	}
	void RegisterHybrid()
	{
		AdpcmFactory = MakeFactory<FPcmAudioInfoHybrid>(Audio::NAME_ADPCM);
		PcmFactory = MakeFactory<FPcmAudioInfoHybrid>(Audio::NAME_PCM);
	}
	void Register(const bool bUseLegacy)
	{
		AdpcmFactory.Reset();
		PcmFactory.Reset();
		if (bUseLegacy)
		{
			RegisterLegacy();
		}
		else
		{
			RegisterHybrid();
		}	
	}
	virtual void StartupModule() override
	{
		static int32 bUseLegacyDecoder = 0;
		static FAutoConsoleVariableRef CVarUseLegacyDecoder(
		TEXT("au.adpcm.UseLegacyDecoder"),
		bUseLegacyDecoder,
		TEXT("0:Hybrid, 1:Legacy"),
		 FConsoleVariableDelegate::CreateLambda([this](IConsoleVariable* Var)
		 {
		 	Register(Var->GetBool());
		 }),
		ECVF_Default);

		Register(!!bUseLegacyDecoder);
	}

	virtual void ShutdownModule() override {}
};

IMPLEMENT_MODULE(FAdpcmAudioDecoderModule, AdpcmAudioDecoder)