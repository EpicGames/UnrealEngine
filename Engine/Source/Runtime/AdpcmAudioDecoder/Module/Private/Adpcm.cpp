// Copyright Epic Games, Inc. All Rights Reserved.

#include "Adpcm.h"
#include "Decoders/ADPCMAudioInfo.h"

namespace ADPCM
{
	using namespace ADPCMPrivate;
	
	template <typename T>
	static void GetAdaptationTable(T(&OutAdaptationTable)[NUM_ADAPTATION_TABLE])
	{
		// Magic values as specified by standard
		static T AdaptationTable[] =
		{
			230, 230, 230, 230, 307, 409, 512, 614,
			768, 614, 512, 409, 307, 230, 230, 230
		};

		FMemory::Memcpy(&OutAdaptationTable, AdaptationTable, sizeof(AdaptationTable));
	}

	struct FAdaptationContext
	{
	public:
		// Adaptation constants
		int32 AdaptationTable[NUM_ADAPTATION_TABLE];
		int32 AdaptationCoefficient1[NUM_ADAPTATION_COEFF];
		int32 AdaptationCoefficient2[NUM_ADAPTATION_COEFF];

		int32 AdaptationDelta;
		int32 Coefficient1;
		int32 Coefficient2;
		int32 Sample1;
		int32 Sample2;

		FAdaptationContext() :
			AdaptationDelta(0),
			Coefficient1(0),
			Coefficient2(0),
			Sample1(0),
			Sample2(0)
		{
			GetAdaptationTable(AdaptationTable);
			GetAdaptationCoefficients(AdaptationCoefficient1, AdaptationCoefficient2);
		}
	};

	FORCEINLINE int16 DecodeNibble(FAdaptationContext& Context, uint8 EncodedNibble)
	{
		int32 PredictedSample = (Context.Sample1 * Context.Coefficient1 + Context.Sample2 * Context.Coefficient2) / 256;
		PredictedSample += SignExtend<int8, 4>(EncodedNibble) * Context.AdaptationDelta;
		PredictedSample = FMath::Clamp(PredictedSample, -32768, 32767);

		// Shuffle samples for the next iteration
		Context.Sample2 = Context.Sample1;
		Context.Sample1 = static_cast<int16>(PredictedSample);
		Context.AdaptationDelta = (Context.AdaptationDelta * Context.AdaptationTable[EncodedNibble]) / 256;
		Context.AdaptationDelta = FMath::Max(Context.AdaptationDelta, 16);

		return Context.Sample1;
	}

	bool DecodeBlock(const uint8* EncodedADPCMBlock, int32 BlockSize, int16* DecodedPCMData)
	{
		FAdaptationContext Context;
		int32 ReadIndex = 0;
		int32 WriteIndex = 0;

		uint8 CoefficientIndex = ReadFromByteStream<uint8>(EncodedADPCMBlock, ReadIndex);
		if (CoefficientIndex >= NUM_ADAPTATION_COEFF)
		{
			UE_LOGF(LogAudio, Error, "Decoding ADPCM block resulted in bad CoefficientIndex (%d). BlockSize: %d, ReadIndex: %d", CoefficientIndex, BlockSize, ReadIndex);
			return false;
		}
		else
		{
			Context.AdaptationDelta = ReadFromByteStream<int16>(EncodedADPCMBlock, ReadIndex);
			Context.Sample1 = ReadFromByteStream<int16>(EncodedADPCMBlock, ReadIndex);
			Context.Sample2 = ReadFromByteStream<int16>(EncodedADPCMBlock, ReadIndex);

			Context.Coefficient1 = Context.AdaptationCoefficient1[CoefficientIndex];
			Context.Coefficient2 = Context.AdaptationCoefficient2[CoefficientIndex];
		}

		// The first two samples are sent directly to the output in reverse order, as per the standard
		DecodedPCMData[WriteIndex++] = Context.Sample2;
		DecodedPCMData[WriteIndex++] = Context.Sample1;

		uint8 EncodedNibblePair = 0;
		uint8 EncodedNibble = 0;
		while (ReadIndex < BlockSize)
		{
			// Read from the byte stream and advance the read head.
			EncodedNibblePair = ReadFromByteStream<uint8>(EncodedADPCMBlock, ReadIndex);

			EncodedNibble = (EncodedNibblePair >> 4) & 0x0F;
			DecodedPCMData[WriteIndex++] = DecodeNibble(Context, EncodedNibble);

			EncodedNibble = EncodedNibblePair & 0x0F;
			DecodedPCMData[WriteIndex++] = DecodeNibble(Context, EncodedNibble);
		}

		return true;
	}
	
	bool DecodeBlock(const uint8* EncodedADPCMBlock, const int32 BlockSize, int16* DecodedPCMData, const int32 DecodedPCMSizeSamples)
	{
		FAdaptationContext Context;
		int32 ReadIndex = 0;
		int32 WriteIndex = 0;

		if (DecodedPCMSizeSamples <= 0)
		{
			UE_LOGF(LogAudio, Error, "Decoding ADPCM block has insufficient space to decode to: DecodedPCMSize=%d", DecodedPCMSizeSamples);
			return false;
		}
		
		const uint8 CoefficientIndex = ReadFromByteStream<uint8>(EncodedADPCMBlock, ReadIndex);
		if (CoefficientIndex >= NUM_ADAPTATION_COEFF)
		{
			UE_LOGF(LogAudio, Error, "Decoding ADPCM block resulted in bad CoefficientIndex (%d). BlockSize: %d, ReadIndex: %d", CoefficientIndex, BlockSize, ReadIndex);
			return false;
		}

		Context.AdaptationDelta = ReadFromByteStream<int16>(EncodedADPCMBlock, ReadIndex);
		Context.Sample1 = ReadFromByteStream<int16>(EncodedADPCMBlock, ReadIndex);
		Context.Sample2 = ReadFromByteStream<int16>(EncodedADPCMBlock, ReadIndex);

		Context.Coefficient1 = Context.AdaptationCoefficient1[CoefficientIndex];
		Context.Coefficient2 = Context.AdaptationCoefficient2[CoefficientIndex];

		// The first two samples are sent directly to the output in reverse order, as per the standard
		DecodedPCMData[WriteIndex++] = Context.Sample2;

		// Handle the single frame case.
		if (WriteIndex < DecodedPCMSizeSamples)
		{
			DecodedPCMData[WriteIndex++] = Context.Sample1;
		}

		// Decode full pairs.
		uint32 RemainingSamples = DecodedPCMSizeSamples - WriteIndex;
		uint32 FullPairs = RemainingSamples / 2;
				
		while (ReadIndex < BlockSize && FullPairs > 0)
		{
			// Read from the byte stream and advance the read head.
			const uint8 EncodedNibblePair = ReadFromByteStream<uint8>(EncodedADPCMBlock, ReadIndex);

			uint8 EncodedNibble = (EncodedNibblePair >> 4) & 0x0F;
			DecodedPCMData[WriteIndex++] = DecodeNibble(Context, EncodedNibble);

			EncodedNibble = EncodedNibblePair & 0x0F;
			DecodedPCMData[WriteIndex++] = DecodeNibble(Context, EncodedNibble);

			FullPairs--;
			RemainingSamples -= 2;
		}

		// Residual half-pair.
		if (RemainingSamples > 0)
		{
			check(RemainingSamples == 1);
			
			// Read from the byte stream and advance the read head.
			const uint8 EncodedNibblePair = ReadFromByteStream<uint8>(EncodedADPCMBlock, ReadIndex);

			uint8 EncodedNibble = (EncodedNibblePair >> 4) & 0x0F;
			DecodedPCMData[WriteIndex++] = DecodeNibble(Context, EncodedNibble);	
		}
		
		return true;
	}
} // end namespace ADPCM

