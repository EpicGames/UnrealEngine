// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "HAL/Platform.h"

namespace ADPCM
{
	// Returns true if successful, false if there was an error in decoding
	bool DecodeBlock(const uint8* EncodedADPCMBlock, const int32 BlockSize, int16* DecodedPCMData, const int32 DecodedPCMSize);
	bool DecodeBlock(const uint8* EncodedADPCMBlock, const int32 BlockSize, int16* DecodedPCMData);
} //namespace ADPCM

namespace ADPCMPrivate
{
	///////////////////////////////////////////////////////`///////////////////
	// Copied from IOS - probably want to split and share
	//////////////////////////////////////////////////////`////////////////////

	template <typename T, uint32 B>
	inline T SignExtend(const T ValueToExtend)
	{
		struct { T ExtendedValue : B; } SignExtender;
		return SignExtender.ExtendedValue = ValueToExtend;
	}

	template <typename T>
	inline T ReadFromByteStream(const uint8* ByteStream, int32& ReadIndex, bool bLittleEndian = true)
	{
		T ValueRaw = 0;

		if (bLittleEndian)
		{
#if PLATFORM_LITTLE_ENDIAN
			for (int32 ByteIndex = 0; ByteIndex < sizeof(T); ++ByteIndex)
#else
			for (int32 ByteIndex = sizeof(T) - 1; ByteIndex >= 0; --ByteIndex)
#endif // PLATFORM_LITTLE_ENDIAN
			{
				ValueRaw |= ByteStream[ReadIndex++] << 8 * ByteIndex;
			}
		}
		else
		{
#if PLATFORM_LITTLE_ENDIAN
			for (int32 ByteIndex = sizeof(T) - 1; ByteIndex >= 0; --ByteIndex)
#else
			for (int32 ByteIndex = 0; ByteIndex < sizeof(T); ++ByteIndex)
#endif // PLATFORM_LITTLE_ENDIAN
			{
				ValueRaw |= ByteStream[ReadIndex++] << 8 * ByteIndex;
			}
		}

		return ValueRaw;
	}

	template <typename T>
	inline void WriteToByteStream(T Value, uint8* ByteStream, int32& WriteIndex, bool bLittleEndian = true)
	{
		if (bLittleEndian)
		{
#if PLATFORM_LITTLE_ENDIAN
			for (int32 ByteIndex = 0; ByteIndex < sizeof(T); ++ByteIndex)
#else
			for (int32 ByteIndex = sizeof(T) - 1; ByteIndex >= 0; --ByteIndex)
#endif // PLATFORM_LITTLE_ENDIAN
			{
				ByteStream[WriteIndex++] = (Value >> (8 * ByteIndex)) & 0xFF;
			}
		}
		else
		{
#if PLATFORM_LITTLE_ENDIAN
			for (int32 ByteIndex = sizeof(T) - 1; ByteIndex >= 0; --ByteIndex)
#else
			for (int32 ByteIndex = 0; ByteIndex < sizeof(T); ++ByteIndex)
#endif // PLATFORM_LITTLE_ENDIAN
			{
				ByteStream[WriteIndex++] = (Value >> (8 * ByteIndex)) & 0xFF;
			}
		}
	}

	template <typename T>
	inline T ReadFromArray(const T* ElementArray, int32& ReadIndex, int32 NumElements, int32 IndexStride = 1)
	{
		T OutputValue = 0;

		if (ReadIndex >= 0 && ReadIndex < NumElements)
		{
			OutputValue = ElementArray[ReadIndex];
			ReadIndex += IndexStride;
		}

		return OutputValue;
	}

	//////////////////////////////////////////////////////////////////////////
	// End of copied section
	//////////////////////////////////////////////////////////////////////////
} // namespace ADPCMPrivate



