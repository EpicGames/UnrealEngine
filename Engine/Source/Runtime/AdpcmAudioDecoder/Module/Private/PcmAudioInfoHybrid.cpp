// Copyright Epic Games, Inc. All Rights Reserved.

#include "PcmAudioInfoHybrid.h"
#include "Adpcm.h"
#include "Containers/StridedView.h"
#include "Decoders/ADPCMAudioInfo.h"
#include "Interfaces/IAudioFormat.h"

namespace AdpcmAudioInfoPrivate
{
	struct FAdpcmImpl final : FPcmAudioInfoHybrid::FHybridImpl
	{
		using FHybridImpl::FHybridImpl;
		
		virtual bool ParseWaveHeader(const FWaveModInfo& WaveInfo, const WaveFormatHeader* FormatHeader, const uint32 HeaderSize) override
		{
			if (!ensure(FormatHeader->wFormatTag == WAVE_FORMAT_ADPCM))
			{
				return false;
			}
			if (HeaderSize < sizeof(ADPCM::ADPCMFormatHeader))
			{
				return false;
			}

			const ADPCM::ADPCMFormatHeader* ADPCMHeader = (const ADPCM::ADPCMFormatHeader*)FormatHeader;
			TotalSamplesPerChannel = ADPCMHeader->SamplesPerChannel;

			// ADPCM starts with 2 uncompressed samples and then the remaining compressed sample data has 2 samples per byte
			UncompressedBlockSamples = ADPCMHeader->wSamplesPerBlock;
			UncompressedBlockSize = UncompressedBlockSamples * sizeof(int16);
			CompressedBlockSize = *WaveInfo.pBlockAlign;

			// Ensure TotalDecodedSize is a even multiple of the compressed block size so that the buffer is not over read on the last block
			TotalBlocks = ((WaveInfo.SampleDataSize + CompressedBlockSize - 1) / CompressedBlockSize);
			TotalBlocksPerChannel = TotalBlocks / NumChannels;
			TotalDecodedSize = TotalBlocks * UncompressedBlockSize;

			Reservoir.SetNum(NumChannels);

			// Encoded format explanation:
			// 
			// The ordering is different if we're streaming or not (for legacy reasons).
			// Streaming format is interleaved blocks.
			// Non-streaming (i.e. ForceInline) is multi-mono blocks.
			// [L][R][C][LFE][LS][RS][L][R][C][LFE][LS][RS] - Streaming format. (blocks are interleaved by channel).
			// [L][L][R][R][C][C][LFE][LFE][LS][LS][RS][RS] - Non streaming. (channels are multi mono).
			// 
			// This affects how we traverse the source data in the decoder. But is abstracted away by the
			// following two stride values:
			//
			// ChannelBlockStride - This is how much in bytes to jump over to get to the next channels data.
			// SrcAdvanceStride - This is how much to advance the source pointer by after reading an entire frame-block (i.e. blocksize * nchannels)

			if (Owner->IsStreaming())
			{
				ChannelBlockStride = CompressedBlockSize;							// 1 block 
				SrcAdvanceStride = CompressedBlockSize * NumChannels;				// 1 block "frame"
			}
			else
			{
				ChannelBlockStride = CompressedBlockSize * TotalBlocksPerChannel;	// 1 entire channels full set of blocks.
				SrcAdvanceStride = CompressedBlockSize;								// 1 block.
			}

			// success.
			return true;
		}

		virtual int32 GetFrameSize() override
		{
			return CompressedBlockSize * NumChannels;
		}

		virtual uint32 GetMaxFrameSizeSamples() const override
		{
			return UncompressedBlockSamples; 
		}

		virtual void SeekToFrame(const uint32 InSeekFrame) override
		{
			using namespace AdpcmAudioInfoPrivate;
			const uint32 SeekFrameClamped = FMath::Clamp<uint32>(InSeekFrame, 0, TotalSamplesPerChannel - 1);

			// Which block is frame in? 
			const uint32 SeekFrameBlockIndex = SeekFrameClamped / UncompressedBlockSamples;
			check(SeekFrameBlockIndex < TotalBlocksPerChannel);

			// Abs offset of first block in AudioData
			const uint32 AbsBlockOffset = SrcAdvanceStride * SeekFrameBlockIndex;

			// Offset in frames of start of block.
			const uint32 BlockFrameStart = SeekFrameBlockIndex * UncompressedBlockSamples;
			check(BlockFrameStart < TotalSamplesPerChannel);

			// Residual samples we need to skip.
			const int32 FramesToSkip = SeekFrameClamped - BlockFrameStart;
			check(FramesToSkip >= 0);
			check(FramesToSkip < (int32)UncompressedBlockSamples);

			// Do the seek.
			TotalFramesDecoded = BlockFrameStart;
			NumFramesToSkip = FramesToSkip;
			Owner->SeekToAbs(AbsBlockOffset, InSeekFrame);

			// Clear reservoir as we've just invalidated it.
			ReservoirStart = 0;
			for (int32 ChannelIndex = 0; ChannelIndex < NumChannels; ++ChannelIndex)
			{
				Reservoir[ChannelIndex].SetNum(0, EAllowShrinking::No);
			}
		}

		virtual void PrepareToLoop() override
		{
			// This is called just before we loop (if the SoundWave is looping).
			// Need to reset or our internal accounting of the final block will be wrong.
			TotalFramesDecoded = 0;
		}

		virtual FDecodeResult Decode(const uint8* CompressedData, const int32 CompressedDataSize, uint8* OutPCMData, const int32 OutputPCMDataSize) override
		{
			uint32 DstSizeFrames = OutputPCMDataSize / sizeof(int16) / NumChannels;;
			int16* Dst = (int16*)OutPCMData;
			uint8 const* Src = (uint8*)CompressedData;
			uint32 SrcSize = CompressedDataSize;

			while (true)
			{
				// Drain reservoir (interleaved copy out)
				if (!DrainReservoir(Dst, DstSizeFrames))
				{
					// Copied out all we can and Dst is full.
					break;
				}

				// Produce full buffer except for the last block which will be limited to how many frames are needed.
				const int32 NumFramesToProduce = FMath::Min(UncompressedBlockSamples, TotalSamplesPerChannel - TotalFramesDecoded);
				if (NumFramesToProduce == 0)
				{
					break;
				}

				// Decode next block of frames into reservoir.
				if (bool bHasError = false; !DecodeNextFrameBlock(Src, SrcSize, NumFramesToProduce, bHasError))
				{
					// Decode either ran of source, or had an error.
					if (bHasError)
					{
						Owner->SetErrorFlag();
					}
					break;
				}

				TotalFramesDecoded += NumFramesToProduce;
			}

			FDecodeResult Result;
			Result.NumPcmBytesProduced = UE_PTRDIFF_TO_INT32((uint8*)Dst - OutPCMData);
			Result.NumAudioFramesProduced = Result.NumPcmBytesProduced / NumChannels / sizeof(uint16);
			Result.NumCompressedBytesConsumed = UE_PTRDIFF_TO_INT32((uint8*)Src - CompressedData);
			return Result;
		}	

		bool DrainReservoir(int16*& OutDst, uint32& OutDstSizeFrames)
		{
			check(NumChannels > 0);
			check(NumChannels == Reservoir.Num());

			// Handle skipping.
			if (NumFramesToSkip > 0)
			{
				const int32 NumFramesBeforeSkip = Reservoir[0].Num() - ReservoirStart;
				check(NumFramesBeforeSkip >= 0);
				const int32 NumFramesToSkipInBuffer = FMath::Min<int32>(NumFramesBeforeSkip, NumFramesToSkip);
				ReservoirStart += NumFramesToSkipInBuffer;
				NumFramesToSkip -= NumFramesToSkipInBuffer;
			}

			const int32 NumSrcFrames = Reservoir[0].Num() - ReservoirStart;
			check(NumSrcFrames >= 0);
			const int32 NumDstFrames = OutDstSizeFrames;
			const int32 NumFramesToCopy = FMath::Min(NumDstFrames, NumSrcFrames);

			// Anything remaining?
			if (NumFramesToCopy > 0)
			{
				switch (NumChannels)
				{
					case 1: // No need for interleave.
					{
						FMemory::Memcpy(OutDst, Reservoir[0].GetData() + ReservoirStart, NumFramesToCopy * sizeof(int16));
						break;
					}
					default:
					{
						// We don't appear to have fast int16 interleave yet, so just hand roll one for now.
						for (int32 Channel = 0; Channel < NumChannels; ++Channel)
						{
							TStridedView<int16> DstChannel = MakeStridedView(sizeof(int16) * NumChannels, OutDst + Channel, NumFramesToCopy);
							const int16* Src = Reservoir[Channel].GetData() + ReservoirStart;
							for (int32 Frame = 0; Frame < NumFramesToCopy; ++Frame)
							{
								DstChannel[Frame] = *Src++;
							}
						}
						break;
					}
				}
				
				// Empty Reservoir
				ReservoirStart += NumFramesToCopy;;
				if (ReservoirStart == Reservoir[0].Num())
				{
					for (int32 Channel = 0; Channel < NumChannels; ++Channel)
					{
						Reservoir[Channel].SetNum(0, EAllowShrinking::No);
					}
					ReservoirStart = 0;
				}
				
				// Subtract what we just wrote from remaining.
				OutDstSizeFrames -= NumFramesToCopy;

				// Advance ptr.
				OutDst += NumFramesToCopy * NumChannels;
			}

			// Return true if there's still more room to copy.
			return OutDstSizeFrames > 0;
		}

		bool DecodeNextFrameBlock(const uint8*& Src, uint32& SrcSize, const int32 NumFramesToProduce, bool& bHasError)
		{
			// Enough Compressed Src to decode all channels? All Src data is multiple of a block size.
			// The final block is still block length, we just decode less of it.
			if ((int32)SrcSize < CompressedBlockSize * NumChannels)
			{
				// Need more data
				return false;
			}

			// Decode next block per channel. (~512 samples, or less. For last block we give a smaller buffer to only produce what we need.)
			for (int32 Channel = 0; Channel < NumChannels; ++Channel)
			{
				const uint8* SrcChannelBlock = Src + (ChannelBlockStride * Channel);

				// Make space.
				TArray<int16>& DstChannel = Reservoir[Channel];
				const int32 NumBefore = DstChannel.Num() - ReservoirStart;
				check(NumBefore >= 0);
				DstChannel.SetNum(NumBefore + NumFramesToProduce, EAllowShrinking::No);

				if (!ADPCM::DecodeBlock(SrcChannelBlock, CompressedBlockSize, DstChannel.GetData() + NumBefore, NumFramesToProduce))
				{
					bHasError = true;
					return false;
				}
			}

			// Advance src.
			SrcSize -= SrcAdvanceStride;
			Src += SrcAdvanceStride;

			// We made some data successfully.
			return true;
		}
		
		uint32 UncompressedBlockSamples = 0;	// Size of a block uncompressed in samples.
		uint32 UncompressedBlockSize = 0;		// Size of a block uncompressed in bytes.
		uint32 CompressedBlockSize = 0;			// Compressed block size in bytes.
		uint32 ChannelBlockStride = 0;			// How much to step (in bytes) over the source data to get to the next channel
		uint32 SrcAdvanceStride = 0;			// How much to advance the source by after consuming a frame block (all channels) worth of data.
		uint32 NumFramesToSkip = 0;				// Number of frames to skip in the reservoir. Used for seeking when we need to discard some of a block
		uint32 TotalBlocks = 0;					// Total number of blocks in the source. Used for sanity checking state. 
		uint32 TotalBlocksPerChannel = 0;		// Total number of blocks per channel in the source. 
		TArray<TArray<int16>> Reservoir;		// Multi-mono buffer, sized by NumChannels, that's the output of each channels block decode.
		uint32 ReservoirStart = 0;				// The start of valid data in each of the reservoir mono buffers. (so we don't need to memmove)
		uint32 TotalFramesDecoded = 0;			// The number of samples decoded so far (per channel)	
	};
	
	struct FPcmImpl final : FPcmAudioInfoHybrid::FHybridImpl
	{
		using FHybridImpl::FHybridImpl;

		virtual bool ParseWaveHeader(const FWaveModInfo& WaveInfo, const WaveFormatHeader* FormatHeader, const uint32 HeaderSize) override
		{
			if (!ensure(FormatHeader->wFormatTag == WAVE_FORMAT_LPCM))
			{
				return false;
			}
	
			// There are no "blocks" in this case
			TotalDecodedSize = WaveInfo.SampleDataSize;
			TotalSamplesPerChannel = TotalDecodedSize / sizeof(uint16) / NumChannels;

			return true;
		}

		virtual FDecodeResult Decode(const uint8* CompressedData, const int32 CompressedDataSize, uint8* OutPCMData, const int32 OutputPCMDataSize) override
		{
			const int32 ConsumedBytes = FMath::Min(OutputPCMDataSize, CompressedDataSize);

			// Copy PCM directly out.
			FMemory::Memcpy(OutPCMData, CompressedData, ConsumedBytes);

			FDecodeResult Result;
			Result.NumAudioFramesProduced = (ConsumedBytes / sizeof(int16)) / NumChannels;
			Result.NumCompressedBytesConsumed = ConsumedBytes;
			Result.NumPcmBytesProduced = ConsumedBytes;
			return Result;
		}

		virtual int32 GetFrameSize() override
		{
			// No framing on PCM, so use the buffer size assuming we can provide that many.
			const uint32 SrcBufferDataSize = Owner->GetSrcBufferDataSize();
			const uint32 SrcBufferOffset = Owner->GetSrcBufferOffset();
			const int32 RemainingInChunk = FMath::Max<int32>(SrcBufferDataSize - SrcBufferOffset, 0);
			const int32 PcmStreamingFrame = FMath::Min<int32>(RemainingInChunk, MONO_PCM_BUFFER_SIZE);

			return PcmStreamingFrame;
		}

		virtual uint32 GetMaxFrameSizeSamples() const override
		{
			return MONO_PCM_BUFFER_SAMPLES;
		}

		virtual void SeekToFrame(const uint32 InSeekFrame) override
		{
			const uint32 SeekFrameClamped = FMath::Clamp<uint32>(InSeekFrame, 0, TotalSamplesPerChannel - 1);
			const uint32 AbsOffset = sizeof(int16) * NumChannels * SeekFrameClamped;
			Owner->SeekToAbs(AbsOffset, SeekFrameClamped);
		}
	};
}// namespace AdpcmAudioInfoPrivate

bool FPcmAudioInfoHybrid::AbsPositionToChunkIndexAndOffset(const uint64 InAbsPosition, uint32& OutChunkIndex, uint32& OutChunkOffset) const
{
	// O(n) search through the possible chunks. This number is pretty small typically.
	// But we could make this faster by just special casing zero, first, rest, and last.
	
	// If we're streaming, grab the streaming wave instance.
	if (const TSharedPtr<const FSoundWaveData>& Wave = GetStreamingSoundWaveData(); Wave.IsValid() && Wave->IsStreaming())
	{
		const int32 NumChunks = Wave->GetNumChunks();
		uint64 AbsChunkOffset = AudioDataOffset;
		for (int32 ChunkIndex = AudioDataChunkIndex; ChunkIndex < NumChunks; ++ChunkIndex)
		{
			const FStreamedAudioChunk& Chunk = Wave->GetChunk(ChunkIndex);
			if (InAbsPosition >= AbsChunkOffset && InAbsPosition < AbsChunkOffset + Chunk.AudioDataSize)
			{
				OutChunkIndex = ChunkIndex;
				OutChunkOffset = IntCastChecked<uint32>(InAbsPosition - AbsChunkOffset);
				return true;
			}
			AbsChunkOffset += Chunk.AudioDataSize;
		}
	}
	return false;
}

bool FPcmAudioInfoHybrid::ParseHeader(const uint8* InSrcBufferData, uint32 InSrcBufferDataSize, FSoundQualityInfo* QualityInfo)
{
	SrcBufferData = InSrcBufferData;
	SrcBufferDataSize = InSrcBufferDataSize;
	SrcBufferOffset = 0;
	CurrentSampleCount = 0;

	// We only have the header if we're streaming, so just parse that.
	const bool bParseJustHeader = GetStreamingSoundWaveData().IsValid() && GetStreamingSoundWaveData()->IsStreaming();

	void* FormatHeader = nullptr;
	FWaveModInfo WaveInfo;
	if (!WaveInfo.ReadWaveInfo(InSrcBufferData, InSrcBufferDataSize, nullptr, bParseJustHeader, &FormatHeader))
	{
		UE_LOGF(LogAudio, Error, "%s WaveInfo.ReadWaveInfo Failed", __func__);
		return false;
	}
	if (!FormatHeader)
	{
		UE_LOGF(LogAudio, Error, "%s WaveInfo.ReadInfo Failed to return a Header", __func__);
		return false;
	}

	// Make sure header is in bounds.
	const uint32 HeaderSize = UE_PTRDIFF_TO_UINT32(WaveInfo.SampleDataStart - InSrcBufferData);
	if (HeaderSize < sizeof(WaveFormatHeader))
	{
		UE_LOGF(LogAudio, Error, "%s HeaderSize=%u", __func__, HeaderSize);
		return false;	
	}

	// Sanity check everything looks sane.
	NumChannels = *WaveInfo.pChannels;
	if (NumChannels <= 0)
	{
		UE_LOGF(LogAudio, Error, "%s NumChannels=%u", __func__, NumChannels);
		return false;
	}
	const uint32 SampleRate = *WaveInfo.pSamplesPerSec;
	if (SampleRate == 0)
	{
		UE_LOGF(LogAudio, Error, "%s SampleRate=%u", __func__, SampleRate);
		return false;	
	}

	// Create the appropriate impl
	Impl = CreateImpl(*WaveInfo.pFormatTag, NumChannels, SampleRate);
	if (!Impl.IsValid())
	{
		UE_LOGF(LogAudio, Error, "%s, Failed to Create Impl for %u", __func__, *WaveInfo.pFormatTag);
		return false;
	}
	
	// Do further format specific stuff.
	if (!Impl->ParseWaveHeader(WaveInfo, static_cast<const WaveFormatHeader*>(FormatHeader), HeaderSize))
	{
		return false;
	}
	
	// Put the read cursor after the header.
	SrcBufferOffset += HeaderSize;
	AudioDataOffset = SrcBufferOffset;
	AudioDataChunkIndex = CurrentChunkIndex; 
	
	// Record the total number of samples.
	TrueSampleCount = Impl->GetTotalSamplesPerChannel() * NumChannels; 
	
	if (QualityInfo)
	{
		QualityInfo->SampleRate = *WaveInfo.pSamplesPerSec;
		QualityInfo->NumChannels = *WaveInfo.pChannels;
		QualityInfo->SampleDataSize = Impl->GetTotalSamplesPerChannel() * NumChannels * sizeof(int16); // Impl->GetTotalDecodedSize();
		QualityInfo->Duration = (float)Impl->GetTotalSamplesPerChannel() / QualityInfo->SampleRate;
	}
	return true;
}

TUniquePtr<FPcmAudioInfoHybrid::FHybridImpl> FPcmAudioInfoHybrid::CreateImpl(const uint8 InFormatId, const int32 InNumChannels, const uint32 InSampleRate)
{
	using namespace AdpcmAudioInfoPrivate;
	switch (InFormatId)
	{
	case WAVE_FORMAT_LPCM:
		return MakeUnique<FPcmImpl>(this, InNumChannels, InSampleRate);
	case WAVE_FORMAT_ADPCM:
		return MakeUnique<FAdpcmImpl>(this,  InNumChannels, InSampleRate );
	default:
		break;
	}
	return {};
}

void FPcmAudioInfoHybrid::SeekToAbs(const uint64 InAbsPosition, const uint64 InSeekFrame)
{
	if (IsStreaming())
	{
		if (uint32 ChunkIndex, ChunkOffset; AbsPositionToChunkIndexAndOffset(InAbsPosition, ChunkIndex, ChunkOffset))
		{
			SetSeekBlockIndex(ChunkIndex);
			SetSeekBlockOffset(ChunkOffset);
			SetCurrentSampleCount(InSeekFrame);
		}
	}
	else // Not streaming.
	{
		SetSrcBufferOffset(GetAudioDataOffset() + InAbsPosition);
		SetCurrentSampleCount(InSeekFrame);
	}	
}

void FPcmAudioInfoHybrid::FHybridImpl::SeekToTime(const float InSeekTime)
{
	check(TotalSamplesPerChannel > 0);
	const uint32 SeekFrame = InSeekTime * SampleRate;
	const uint32 SeekFrameClamped = FMath::Clamp<uint32>( SeekFrame, 0, TotalSamplesPerChannel - 1);
	SeekToFrame(SeekFrameClamped);
}


