// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AudioDecompress.h"
#include "Decoders/ADPCMAudioInfo.h"

class FPcmAudioInfoHybrid : public IStreamedCompressedInfo
{
public:
	FPcmAudioInfoHybrid() = default;
	virtual ~FPcmAudioInfoHybrid() override = default;

	class FHybridImpl
	{
	public:
		UE_NONCOPYABLE(FHybridImpl)
		
		FHybridImpl(FPcmAudioInfoHybrid* InOwner, const int32 InNumChannels, const uint32 InSampleRate)
			: Owner(InOwner)
			, NumChannels(InNumChannels)
			, SampleRate(InSampleRate)
		{}
						
		virtual ~FHybridImpl() = default;
		virtual void PrepareToLoop() {}
		virtual bool ParseWaveHeader(const FWaveModInfo& WaveInfo, const WaveFormatHeader* FormatHeader, const uint32 HeaderSize) = 0;
		virtual int32 GetFrameSize() = 0;
		virtual uint32 GetMaxFrameSizeSamples() const = 0;
		virtual void SeekToFrame(const uint32 InSeekFrame) = 0;
		virtual void SeekToTime(const float InSeekTime);
		virtual FDecodeResult Decode(const uint8* CompressedData, const int32 CompressedDataSize, uint8* OutPCMData, const int32 OutputPCMDataSize) = 0;
		
		uint32 GetTotalDecodedSize() const { return TotalDecodedSize; }
		uint32 GetTotalSamplesPerChannel() const { return TotalSamplesPerChannel; }
		
	protected:
		FPcmAudioInfoHybrid* Owner = nullptr;
		int32 NumChannels = 0;
		uint32 TotalDecodedSize = 0;
		uint32 SampleRate = 0;
		uint32 TotalSamplesPerChannel = 0;			// Number of samples per channel, used to detect when an audio waveform has ended
	};
	
protected:

	/** Parse the header information from the input source buffer data. This is dependent on compression format. */
	virtual bool ParseHeader(const uint8* InSrcBufferData, uint32 InSrcBufferDataSize, FSoundQualityInfo* QualityInfo) override;

	/** Create the compression format dependent decoder object. */
	virtual bool CreateDecoder() override { return true; }

	virtual void PrepareToLoop() override
	{
		Impl->PrepareToLoop();
	}
	virtual int32 GetFrameSize() override
	{
		return Impl->GetFrameSize();
	}
	virtual uint32 GetMaxFrameSizeSamples() const override
	{
		return Impl->GetMaxFrameSizeSamples();
	}
	virtual void SeekToFrame(const uint32 InSeekFrame) override
	{
		return Impl->SeekToFrame(InSeekFrame);
	}
	virtual void SeekToTime(const float InSeekTime) override
	{
		return Impl->SeekToTime(InSeekTime);
	}
	virtual FDecodeResult Decode(const uint8* CompressedData, const int32 CompressedDataSize, uint8* OutPCMData, const int32 OutputPCMDataSize) override
	{
		return Impl->Decode(CompressedData, CompressedDataSize, OutPCMData, OutputPCMDataSize);	
	}

public:
	bool IsStreaming() const
	{
		return GetStreamingSoundWaveData().IsValid() && GetStreamingSoundWaveData()->IsStreaming();
	}
	void SeekToAbs(const uint64 InAbsPosition, const uint64 InSeekFrame);	
	void SetErrorFlag() const { bHasError = true; }
	
	uint32 GetAudioDataOffset() const { return AudioDataOffset; }
	uint32 GetSrcBufferOffset() const { return SrcBufferOffset; }
	uint32 GetSrcBufferDataSize() const { return SrcBufferDataSize; }
	
private:
	TUniquePtr<FHybridImpl> CreateImpl(const uint8 InFormatId, const int32 InNumChannels, const uint32 InSampleRate);
	bool AbsPositionToChunkIndexAndOffset(const uint64 AbsPosition, uint32& ChunkIndex, uint32& ChunkOffset) const;
	
	void SetSeekBlockIndex(const uint32 InStreamSeekBlockIndex)
	{
		if (ensure(GetStreamingSoundWaveData().IsValid() && GetStreamingSoundWaveData()->IsStreaming()
			&& InStreamSeekBlockIndex < GetStreamingSoundWaveData()->GetNumChunks()))
		{
			StreamSeekBlockIndex = InStreamSeekBlockIndex;
		}
	};
	void SetSeekBlockOffset(const int32 InStreamSeekBlockOffset)
	{
		StreamSeekBlockOffset = InStreamSeekBlockOffset;
	};
	void SetCurrentSampleCount(const uint32 InSampleCount) { CurrentSampleCount = InSampleCount; }
	void SetSrcBufferOffset(const uint32 InSrcBufferOffset)
	{
		ensure(InSrcBufferOffset < SrcBufferDataSize); 
		SrcBufferOffset = InSrcBufferOffset;
	}

	friend class FHybridImpl;
	TUniquePtr<FHybridImpl> Impl;
};
