// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	ADPCMAudioInfo.h: Unreal audio ADPCM decompression interface object.
=============================================================================*/

#pragma once

#include "CoreMinimal.h"
#include "Audio.h"
#include "AudioDecompress.h"
#include "Sound/SoundWave.h"
#include "ContentStreaming.h"

#define NUM_ADAPTATION_TABLE 16
#define NUM_ADAPTATION_COEFF 7

#define WAVE_FORMAT_LPCM  1
#ifndef WAVE_FORMAT_ADPCM
#define WAVE_FORMAT_ADPCM 2
#endif

#if PLATFORM_SUPPORTS_PRAGMA_PACK
#pragma pack(push, 2)
#endif
	struct WaveFormatHeader
	{
		uint16 wFormatTag;      // Format type: 1 = PCM, 2 = ADPCM
		uint16 nChannels;       // Number of channels (i.e. mono, stereo...).
		uint32 nSamplesPerSec;  // Sample rate. 44100 or 22050 or 11025  Hz.
		uint32 nAvgBytesPerSec; // For buffer estimation  = sample rate * BlockAlign.
		uint16 nBlockAlign;     // Block size of data = Channels times BYTES per sample.
		uint16 wBitsPerSample;  // Number of bits per sample of mono data.
		uint16 cbSize;          // The count in bytes of the size of extra information (after cbSize).
	};
#if PLATFORM_SUPPORTS_PRAGMA_PACK
#pragma pack(pop)
#endif

namespace ADPCM
{

	template <typename T>
	static void GetAdaptationCoefficients(T(& OutAdaptationCoefficient1)[NUM_ADAPTATION_COEFF], T(& OutAdaptationCoefficient2)[NUM_ADAPTATION_COEFF])
	{
		// Magic values as specified by standard
		static T AdaptationCoefficient1[] =
		{
			256, 512, 0, 192, 240, 460, 392
		};
		static T AdaptationCoefficient2[] =
		{
			0, -256, 0,  64, 0, -208, -232
		};

		FMemory::Memcpy(&OutAdaptationCoefficient1, AdaptationCoefficient1, sizeof(AdaptationCoefficient1));
		FMemory::Memcpy(&OutAdaptationCoefficient2, AdaptationCoefficient2, sizeof(AdaptationCoefficient2));
	}

#if PLATFORM_SUPPORTS_PRAGMA_PACK
#pragma pack(push, 2)
#endif
	struct ADPCMFormatHeader
	{
		WaveFormatHeader	BaseFormat;
		uint16				wSamplesPerBlock;
		uint16				wNumCoef;
		int16				aCoef[2 * NUM_ADAPTATION_COEFF];
		uint32				SamplesPerChannel;	// This is the exact samples per channel for sample precise looping

		ADPCMFormatHeader()
		{
			FMemory::Memzero(this, sizeof(ADPCMFormatHeader));
			int16 AdaptationCoefficient1[NUM_ADAPTATION_COEFF];
			int16 AdaptationCoefficient2[NUM_ADAPTATION_COEFF];
			GetAdaptationCoefficients(AdaptationCoefficient1, AdaptationCoefficient2);

			// Interlace the coefficients as pairs
			for (int32 Coeff = 0, WriteIndex = 0; Coeff < NUM_ADAPTATION_COEFF; Coeff++)
			{
				aCoef[WriteIndex++] = AdaptationCoefficient1[Coeff];
				aCoef[WriteIndex++] = AdaptationCoefficient2[Coeff];
			}
		}
	};
#if PLATFORM_SUPPORTS_PRAGMA_PACK
#pragma pack(pop)
#endif

};

class FADPCMAudioInfo : public ICompressedAudioInfo
{
public:
	ADPCMAUDIODECODER_API FADPCMAudioInfo(void);
	ADPCMAUDIODECODER_API virtual ~FADPCMAudioInfo(void);
	// ICompressedAudioInfo Interface
	ADPCMAUDIODECODER_API virtual bool ReadCompressedInfo(const uint8* InSrcBufferData, uint32 InSrcBufferDataSize, struct FSoundQualityInfo* QualityInfo);
	ADPCMAUDIODECODER_API virtual bool ReadCompressedData(uint8* Destination, bool bLooping, uint32 BufferSize);
	ADPCMAUDIODECODER_API virtual void SeekToTime(const float SeekTime);
	ADPCMAUDIODECODER_API virtual void SeekToFrame(const uint32 SeekFrame);
	ADPCMAUDIODECODER_API virtual void ExpandFile(uint8* DstBuffer, struct FSoundQualityInfo* QualityInfo);
	virtual void EnableHalfRate(bool HalfRate) {};
	virtual uint32 GetSourceBufferSize() const { return SrcBufferDataSize; }
	virtual bool UsesVorbisChannelOrdering() const { return false; }
	virtual int GetStreamBufferSize() const;

	// Additional overrides for streaming
	virtual bool SupportsStreaming() const override {return true;}
	virtual bool StreamCompressedInfoInternal(const TSharedRef<const FSoundWaveData>& InWaveData, struct FSoundQualityInfo* QualityInfo) override;
	virtual bool StreamCompressedData(uint8* Destination, bool bLooping, uint32 BufferSize, int32& OutNumBytesStreamed) override;
	virtual int32 GetCurrentChunkIndex() const override
	{
		return CurrentChunkIndex;
	}
	virtual int32 GetCurrentChunkOffset() const override {return CurrentChunkBufferOffset;}
	virtual bool ReleaseStreamChunk(bool bBlockUntilReleased) override;
	// End of ICompressedAudioInfo Interface

	uint16 GetFormatTag()
	{
		if (WaveInfo.pFormatTag)
		{
			return *WaveInfo.pFormatTag;
		}
		else
		{
			return 0;
		}
	}

	int32 GetNumChannels()
	{
		return NumChannels;
	}

private:

	// Wrapper function that returns a pointer to the currently used compressed data.
	// If a non-zero chunk is requested, this function also aquires a reference to that chunk
	// until we move on to a different chunk.
	const uint8* GetLoadedChunk(const FSoundWaveData& InSoundWaveData, uint32 ChunkIndex, uint32& OutChunkSize);

	int32 NumConsecutiveReadFailiures;

	FWaveModInfo WaveInfo;
	const uint8*	SrcBufferData;
	uint32			SrcBufferDataSize;

	uint32	UncompressedBlockSize;
	uint32	CompressedBlockSize;
	uint32	BlockSize;
	int32	StreamBufferSize;
	uint32	TotalDecodedSize;
	int32	NumChannels;
	int32	Format;

	uint32 PreviouslyRequestedChunkIndex;

	uint8*			UncompressedBlockData;			// This holds the current block of compressed data for all channels
	uint32			CurrentUncompressedBlockSampleIndex;	// This is the sample index within the current uncompressed block data
	uint32			CurrentChunkIndex;				// This is the index that is currently being used, needed by streaming engine to make sure it stays loaded and the next chunk gets preloaded
	uint32			CurrentChunkBufferOffset;		// This is this byte offset within the current chunk, used by streaming engine to prioritize a load if more then half way through current chunk
	uint32			CurrentChunkDataSize;			// The size of the current chunk, the first chunk is bigger to accommodate the header info
	uint32			TotalSamplesStreamed;			// The number of samples streamed so far (per channel)
	uint32			TotalSamplesPerChannel;			// Number of samples per channel, used to detect when an audio waveform has ended
	uint32			SamplesPerBlock;				// The number of samples per block
	uint32			FirstChunkSampleDataOffset;		// The size of the header in the first chunk, used to skip over it when looping or starting the sample over
	uint32          FirstChunkSampleDataIndex;
	const uint8*	CurCompressedChunkData;			// A pointer to the current chunk of data
	FAudioChunkHandle CurCompressedChunkHandle;     // Shared reference to the current chunk of data.
	FCriticalSection CurCompressedChunkHandleCriticalSection;
	bool bDecompressorReleased;                     // When the chunk is released, we raise this to true, then early exit on future decodes.

	uint32			CurrentCompressedBlockIndex;		// For non disk streaming - the current compressed block in the compressed source data
	uint32			TotalCompressedBlocksPerChannel;	// For non disk streaming - the total number of compressed blocks per channel
	uint8			bNewSeekRequest : 1;				// Whether or not there is a new seek request to be processed 
	uint8			bSeekPendingRead : 1;				// Whether or not a requested seek is pending a read
	uint8           bSeekedFowardToNextChunk : 1;       // If this is true, we have already seeked forward to the next chunk while waiting for the current chunk of audio to load.

private:
	void ProcessSeekRequest();
	void SeekToFrameInternal(const uint32 InSeekFrame);
	void ResetSeekState();

	std::atomic<uint32> TargetSeekFrame;

	FCriticalSection StreamSeekCriticalSection;
};